[CrashReportClient]
CrashReportClientVersion=1.0
bAllowToBeContacted=true
bSendLogFile=true
CanSendWhenUIFailedToInitialize=true
UIInitRetryCount=10
UIInitRetryInterval=2.0

[/Script/Engine.NetworkSettings]
n.VerifyPeer=false

[/Script/UdpMessaging.UdpMessagingSettings]
EnableTransport=True
MulticastTimeToLive=0
